// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"

#include "IStageDataProvider.h"
#include "Modules/ModuleManager.h"

/** Name of the DataProvider modular feature */
FName IStageDataProvider::ModularFeatureName = "ModularFeature_StageDataProvider";


IMPLEMENT_MODULE(FDefaultModuleImpl, StageDataCore);

