// Copyright Epic Games, Inc. All Rights Reserved.

#include "StageMessages.h"

#include "Misc/App.h"

FString FCriticalStateProviderMessage::ToString() const
{
	switch (State)
	{
		case EStageCriticalStateEvent::Enter:
		{
			return FString::Printf(TEXT("%s: Entered critical state"), *SourceName.ToString());
		}
		case EStageCriticalStateEvent::Exit:
		default:
		{
			return FString::Printf(TEXT("%s: Exited critical state"), *SourceName.ToString());
		}
	}
}

FString FAssetLoadingStateProviderMessage::ToString() const
{
	switch (LoadingState)
	{
		case EStageLoadingState::PreLoad:
		{
			return FString::Printf(TEXT("Started loading asset: %s"), *AssetName);
		}
		case EStageLoadingState::PostLoad:
		default:
		{
			return FString::Printf(TEXT("Finished loading asset: %s"), *AssetName);
		}
	}
}

