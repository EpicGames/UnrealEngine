// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

#include "CoreMinimal.h"
#include "Misc/App.h"
#include "StageMessages.h"

namespace UE::StageProviderUtils
{
	static const FQualifiedFrameTime InvalidTime = FQualifiedFrameTime(FFrameTime(FFrameNumber(-1)), FFrameRate(-1, -1));
}

struct FStageDataBaseMessage;

/**
 * Interface for the data provider modular feature used in engine.
 * Also contains methods for modules that would like to send data about stage status
 */
class STAGEDATACORE_API IStageDataProvider : public IModularFeature
{
public:
	virtual ~IStageDataProvider() {}

	static FName ModularFeatureName;

	//~ Begin static methods for data provider users

public:

	/** Generic send message to support constructor parameters directly and temp object created */
	template<typename MessageType, typename... Args>
	static bool SendMessage(EStageMessageFlags Flags, Args&&... args)
	{
#if ENABLE_STAGEMONITOR_LOGGING
		MessageType TempObj(Forward<Args>(args)...);
		return SendMessage(MoveTempIfPossible(TempObj), Flags);
#endif

		return false;
	}

	/** Specialized send message for lvalue type without a temp object */
	template<typename MessageType, typename... Args>
	static bool SendMessage(EStageMessageFlags Flags, MessageType& Message)
	{
#if ENABLE_STAGEMONITOR_LOGGING
		return SendMessage(Forward<MessageType>(Message), Flags);
#endif

		return false;
	}

	/** Specialized send message for rvalue type without a temp object */
	template<typename MessageType, typename... Args>
	static bool SendMessage(EStageMessageFlags Flags, MessageType&& Message)
	{
#if ENABLE_STAGEMONITOR_LOGGING
		return SendMessage(Forward<MessageType>(Message), Flags);
#else
		return false;
#endif
	}

private:

	/** Internal method where all send messages are funneled before being sent to the data provider instance */
	template<typename MessageType>
	static bool SendMessage(MessageType&& Message, EStageMessageFlags InFlags)
	{
		//To simply API, modular feature checkup is hidden here. Could add preprocessors to disable this entirel
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if (ModularFeatures.IsModularFeatureAvailable(IStageDataProvider::ModularFeatureName))
		{
			IStageDataProvider* StageDataProvider = &IModularFeatures::Get().GetModularFeature<IStageDataProvider>(IStageDataProvider::ModularFeatureName);

			static_assert(TIsDerivedFrom<MessageType, FStageProviderEventMessage>::IsDerived || TIsDerivedFrom<MessageType, FStageProviderPeriodicMessage>::IsDerived, "MessageType must be a FStageProviderEventMessage or FStageProviderPeriodicMessage derived UStruct.");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Timestamp the outgoing message.
			TOptional<FQualifiedFrameTime> CurrentFrameTime = FAppTime::Get_UnsafeNoCheck().CurrentFrameTime;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Message.DateTime = FDateTime::Now();

			if (CurrentFrameTime.IsSet())
			{
				Message.FrameTime = CurrentFrameTime.GetValue();
			}
			else
			{
				Message.FrameTime = UE::StageProviderUtils::InvalidTime;
			}

			return StageDataProvider->SendMessageInternal(&Message, MessageType::StaticStruct(), InFlags);
		}

		return false;
	}

	//~ End static methods for data provider users

protected:

	/** Method to actually send message from a data provider to monitors */
	virtual bool SendMessageInternal(FStageDataBaseMessage* Payload, UScriptStruct* Type, EStageMessageFlags InFlags) = 0;
};
