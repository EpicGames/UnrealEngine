// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/QualifiedFrameTime.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "StageMessages.generated.h"

/** Global define to disable any data provider sending logic */
#define ENABLE_STAGEMONITOR_LOGGING (1 && !NO_LOGGING && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#define UE_API STAGEDATACORE_API


/** Message flags configuring how a message is sent */
UENUM()
enum class EStageMessageFlags : uint8
{
	None = 0,

	/** Sends this message as reliable, to make sure it's received by the receivers */
	Reliable = 1 << 0,
};
ENUM_CLASS_FLAGS(EStageMessageFlags);

/** States that a DataProvider can be in */
UENUM()
enum class EStageDataProviderState : uint8
{
	/** Actively receiving messages from */
	Active,
	/** Timeout intervaled occured between provider messages */
	Inactive,
	/** Provider closed */
	Closed,
};


/** Different events associated with stage critical state */
UENUM()
enum class EStageCriticalStateEvent : uint8
{
	/** Critical state has been entered */
	Enter,
	/** Critical state has been exited */
	Exit,
};

/** Different events associated with stage critical state */
UENUM()
enum class EStageLoadingState : uint8
{
	/** Asset loading has started. */
	PreLoad,
	/** Asset loading has finished. */
	PostLoad
};

/**
 * Holds descriptive information about that data providers. 
 * Information that won't change for a session
 * Used when monitor and provider connects
 */
USTRUCT(MinimalAPI)
struct FStageInstanceDescriptor
{
	GENERATED_BODY()

public:

public:

	/** Machine name read from FPlatformProcess::ComputerName() */
	UPROPERTY()
	FString MachineName;

	/** ProcessId read from FPlatformProcess::GetCurrentProcessId */
	UPROPERTY()
	uint32 ProcessId = 0;
	
	/** Simple stringified view of the roles of that instance */
	UPROPERTY()
	FString RolesStringified;

	/** Friendly name for this Unreal instance. If empty, this will be MachineName - ProcessId */
	UPROPERTY()
	FName FriendlyName;

	/** Session Id that may be used to differentiate different sessions on the network */
	UPROPERTY()
	int32 SessionId = INDEX_NONE;
};

/**
 *  Base structure for all stage monitoring messages
 */
USTRUCT(MinimalAPI)
struct FStageDataBaseMessage
{
	GENERATED_BODY()

public:
	virtual ~FStageDataBaseMessage() = default;
	
public:

	/** 
	 * Provision for versioning if we need to differentiate version of messages.
	 * All stage machines should be running the same version of Unreal but if it's not
	 * the case, having a version in the message will be useful to know about it.
	 */
	UPROPERTY()
	int32 StageMessageVersion = 1;

	/** Identifier of this instance */
	UPROPERTY()
	FGuid Identifier;
};

/** Monitor only messages - listened by providers */


/**
 *  Base structure for all monitor messages
 */
USTRUCT(MinimalAPI)
struct FStageMonitorBaseMessage : public FStageDataBaseMessage
{
	GENERATED_BODY()
};

/**
 *  Message broadcasted periodically by the monitor to discover new providers 
 */
USTRUCT(MinimalAPI)
struct FStageProviderDiscoveryMessage : public FStageMonitorBaseMessage
{
	GENERATED_BODY()

public:
	FStageProviderDiscoveryMessage() = default;
	FStageProviderDiscoveryMessage(FStageInstanceDescriptor&& InDescriptor)
		: Descriptor(MoveTemp(InDescriptor))
		{}

public:

	/** Detailed description of that monitor */
	UPROPERTY()
	FStageInstanceDescriptor Descriptor;
};

/**
 *  Message sent when monitor is going down to let know linked providers
 */
USTRUCT(MinimalAPI)
struct FStageMonitorCloseMessage : public FStageMonitorBaseMessage
{
	GENERATED_BODY()
};

/** 
 * Base Provider messages listened by monitors
 */
USTRUCT(MinimalAPI)
struct FStageProviderMessage : public FStageDataBaseMessage
{
	GENERATED_BODY()

public:
	/** Method to override if a detailed description is desired in the monitor event viewer */
	virtual FString ToString() const { return FString(); }

public:
	
	/** FrameTime of the sender. It's expected to have all stage machines using the same timecode provider to play in the same referential*/
	UPROPERTY()
	FQualifiedFrameTime FrameTime;

	/** DateTime of the sender. Used to keep track of messages order through Timecode rollover. */
	UPROPERTY()
	FDateTime DateTime;
};

/**
 * Base Provider messages that are events
 */
USTRUCT(MinimalAPI)
struct FStageProviderEventMessage : public FStageProviderMessage
{
	GENERATED_BODY()
};

/**
 * Base Provider messages that are periodic
 */
USTRUCT(MinimalAPI)
struct FStageProviderPeriodicMessage : public FStageProviderMessage
{
	GENERATED_BODY()
};

/**
 *  Message sent by Providers to notify monitors they are closing down
 */
USTRUCT(MinimalAPI)
struct FStageProviderCloseMessage : public FStageProviderEventMessage
{
	GENERATED_BODY()
};

/**
 * Response to a received discovery message sent by providers
 */
USTRUCT(MinimalAPI)
struct FStageProviderDiscoveryResponseMessage : public FStageProviderEventMessage
{
	GENERATED_BODY()
	
public:
	UE_API FStageProviderDiscoveryResponseMessage() = default;
	UE_API FStageProviderDiscoveryResponseMessage(FStageInstanceDescriptor&& InDescriptor)
		: Descriptor(MoveTemp(InDescriptor))
		{}


public:

	/** Detailed description of that provider */
	UPROPERTY()
	FStageInstanceDescriptor Descriptor;
};

/**
 * Message sent to notify about critical state change.
 */
USTRUCT(MinimalAPI)
struct FCriticalStateProviderMessage : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:

	FCriticalStateProviderMessage() = default;

	FCriticalStateProviderMessage(EStageCriticalStateEvent InEvent, FName InSourceName)
		: State(InEvent), SourceName(InSourceName)
	{
	}

	UE_API virtual FString ToString() const override;

	/** Event for this critical state */
	UPROPERTY(VisibleAnywhere, Category = "CriticalState")
	EStageCriticalStateEvent State = EStageCriticalStateEvent::Enter;

	/** Source of the critical state. i.e. TakeName, CustomRecorder, etc... */
	UPROPERTY(VisibleAnywhere, Category = "CriticalState")
	FName SourceName;
};

/**
 * Message sent to indicate that the node has entered or exited a loading state.
 */
USTRUCT(MinimalAPI)
struct FAssetLoadingStateProviderMessage : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FAssetLoadingStateProviderMessage() = default;

	FAssetLoadingStateProviderMessage(EStageLoadingState InState, const FString& InAssetName)
		: LoadingState(InState), AssetName(InAssetName)
	{}

	UE_API virtual FString ToString() const override;

	/** Event for this critical state */
	UPROPERTY(VisibleAnywhere, Category = "AssetLoading")
	EStageLoadingState LoadingState = EStageLoadingState::PreLoad;

	/** Name of the asset currently loading. */
	UPROPERTY(VisibleAnywhere, Category = "AssetLoading")
	FString AssetName;
};
 
#undef UE_API