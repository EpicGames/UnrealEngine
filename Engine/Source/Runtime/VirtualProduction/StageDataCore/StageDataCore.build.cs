// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StageDataCore : ModuleRules
{
	public StageDataCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
            {
            }
		);
	}
}
