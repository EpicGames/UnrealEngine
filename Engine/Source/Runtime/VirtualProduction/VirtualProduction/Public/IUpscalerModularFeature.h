// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

struct FInstancedPropertyBag;
struct FSceneViewExtensionIsActiveFunctor;

class FSceneViewFamilyContext;
class FSceneViewFamily;
class FSceneView;

namespace UE::VirtualProduction
{
	/**
	* The upscaler modular feature parameters.
	*/
	struct FUpscalerModularFeatureParameters
	{
		// Upscaler screen percentage vaue.
		float UpscalerScreenPercentage = 1.0f;

		// Secondary screen percentage value.
		float SecondaryScreenPercentage = 1.0f;
	};

	/**
	 * Interface for a modular feature of a Upscaler.
	 *
	 * This interface provides a way to interact with the upscaler without requiring the presence
	 * of their specific plugins. It allows modular features to be integrated and accessed within the
	 * rendering pipeline in a flexible and decoupled manner.
	 *
	 * Classes implementing this interface can define custom behavior and settings for Scene View Extensions
	 * while maintaining compatibility with other parts of the Unreal Engine system.
	 */
	class IUpscalerModularFeature
		: public IModularFeature
	{
	public:
		virtual ~IUpscalerModularFeature() = default;

		/** The unique modular feature name. */
		static constexpr const TCHAR* ModularFeatureName = TEXT("UpscalerModularFeature");

	public:
		/**
		 * Returns the unique identifier name for this feature.
		 */
		virtual const FName& GetName() const = 0;

		/**
		 * Gets the display name shown in the UI.
		 */
		virtual const FText& GetDisplayName() const = 0;

		/**
		 * Returns a hint text for the feature.
		 */
		virtual const FText& GetTooltipText() const = 0;

	public:
		/**
		 * Determines whether the feature is currently enabled and available for use.
		 * This can depend on config, platform, or runtime logic.
		 */
		virtual bool IsFeatureEnabled() const = 0;

		/**
		* Adds a functor and returns a GUID to identify it.
		* 
		* @param IsActiveFunction - the functor with custom logic for the upscaler ViewExtension
		* 
		* @return true on success
		*/
		virtual bool AddSceneViewExtensionIsActiveFunctor(const FSceneViewExtensionIsActiveFunctor& IsActiveFunction) = 0;

		/** 
		* Removes a functor by GUID.
		* 
		* @param FunctorGuid - use value from the FSceneViewExtensionIsActiveFunctor::GetGuid()
		*
		* @return true if the functor exists and has been deleted
		*/
		virtual bool RemoveSceneViewExtensionIsActiveFunctor(const FGuid& FunctorGuid) = 0;

		/**
		* Get default settings for this upscaler
		*
		* @param OuUpscalerSettings - (out) default settings
		* 
		* @return true if successed.
		*/
		virtual bool GetSettings(FInstancedPropertyBag& OuUpscalerSettings) const
		{
			return false;
		}

		/**
		* Setup SceneView for this upscaler.
		* Upscalers should configure the view properties (e.g. AntiAliasingMethod) for their rendering pipeline (temporal/spatial/etc.).
		*
		* @param InUpscalerSettings - (in) upscaler settings which we want to use. This settings should be associated with the InOutViews.
		* @param InOutView          - (in) scene view to that will be configured.
		*/
		virtual void SetupSceneView(
			const FInstancedPropertyBag& InUpscalerSettings,
			FSceneView& InOutView) = 0;

		/**
		* Configure ViewFamily and it Views for specific settings and use these settings for the views.
		* Note: Expected that these parameters should be configured:
		*   InOutViewFamily.SetScreenPercentageInterface();
		*   InOutViewFamily.SecondaryViewFraction;
		* 241300
		* Some features from the settings may require additional customization in the view family or view structures.
		*
		* @param InUpscalerSettings - (in) upscaler settings which we want to use. This settings should be associated with the InOutViews.
		* @param InUpscalerParam    - (in) upscaler parameters that is used to configure view family.
		* @param InOutViewFamily    - (in, out) The view family that will be configured.
		*/
		virtual bool PostConfigureViewFamily(
			const FInstancedPropertyBag& InUpscalerSettings,
			const FUpscalerModularFeatureParameters& InUpscalerParam,
			FSceneViewFamilyContext& InOutViewFamily) = 0;
	};
};
